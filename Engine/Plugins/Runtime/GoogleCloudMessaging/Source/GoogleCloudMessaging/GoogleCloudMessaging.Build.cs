// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class GoogleCloudMessaging : ModuleRules
	{
		public GoogleCloudMessaging(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"Launch"
				});

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}

			if (Target.Platform == UnrealTargetPlatform.Android)
			{
				string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
				AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "GoogleCloudMessaging_UPL.xml"));
			}

			PublicIncludePathModuleNames.Add("Launch");
		}
	}
}
