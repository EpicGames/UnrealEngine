<?xml version="1.0" encoding="utf-8"?>
<!-- Google Cloud Messaging plugin additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
  <!-- init section is always evaluated once per architecture -->
  <init>
    <log text="Google Cloud Messaging init"/>
    <!-- client sender ID -->
    <setStringFromProperty result="GCMClientSenderID" ini="Engine" section="/Script/AndroidRuntimeSettings.AndroidRuntimeSettings" property="GCMClientSenderID" default="" />
    <setBoolIsEqual result="GCMClientSenderInvalid" arg1="$S(GCMClientSenderID)" arg2=""/>
    <setBoolNot result="GCMEnabled" source="$B(GCMClientSenderInvalid)"/>
    <log text="GCMClientSenderID set: $B(GCMEnabled)"/>
	</init>

	<buildGradleAdditions>
		<if condition="GCMEnabled">
			<true>
				<insert>
dependencies {
	implementation('com.google.android.gms:play-services-gcm:17.0.0')
}
				</insert>
			</true>
		</if>
	</buildGradleAdditions>

	<!-- optional updates applied to AndroidManifest.xml -->
  <androidManifestUpdates>
    <!-- only modify manifest if enabled -->
    <if condition="GCMEnabled">
      <true>
        <!-- get the package name from manifest -->
        <setStringFromAttribute result="AppPackageName" tag="manifest" name="package"/>

        <addPermission android:name="com.google.android.c2dm.permission.RECEIVE" />
        <addElements tag="application">
          <service
            android:name="com.epicgames.unreal.RemoteNotificationsListener"
            android:exported="false" >
            <intent-filter>
              <action android:name="com.google.android.c2dm.intent.RECEIVE" />
            </intent-filter>
          </service>
          <service
            android:name="com.epicgames.unreal.RemoteNotificationsInstanceIDListener"
            android:exported="false">
            <intent-filter>
              <action android:name="com.google.android.gms.iid.InstanceID"/>
            </intent-filter>
          </service>
          <service
            android:name="com.epicgames.unreal.RemoteNotificationsRegistrationIntentService"
            android:exported="false">
          </service>
        </addElements>

        <!-- create receiver -->
        <setElement result="GCMReceiver" value="receiver" />
        <addAttribute tag="$GCMReceiver" name="android:name" value="com.google.android.gms.gcm.GcmReceiver" />
        <addAttribute tag="$GCMReceiver" name="android:exported" value="true" />
        <addAttribute tag="$GCMReceiver" name="android:permission" value="com.google.android.c2dm.permission.SEND" />

        <!-- create intent filter -->
        <setElement result="GCMReceiverIntentFilter" value="intent-filter" />

        <!-- create action -->
        <setElement result="GCMIntentFilterAction" value="action" />
        <addAttribute tag="$GCMIntentFilterAction" name="android:name" value="com.google.android.c2dm.intent.RECEIVE" />

        <!-- create category -->
        <setElement result="GCMIntentFilterCategory" value="category" />
        <addAttribute tag="$GCMIntentFilterCategory" name="android:name" value="$S(AppPackageName)" />

        <!-- add action & category to intent-filter -->
        <addElement tag="$GCMReceiverIntentFilter" name="GCMIntentFilterAction" />
        <addElement tag="$GCMReceiverIntentFilter" name="GCMIntentFilterCategory" />

        <!-- add intent-filter to receiver -->
        <addElement tag="$GCMReceiver" name="GCMReceiverIntentFilter" />

        <!-- add receiver to application -->
        <addElement tag="application" name="GCMReceiver" />

      </true>
    </if>
  </androidManifestUpdates>

  <!-- optional additions to proguard
	<proguardAdditions>
	</proguardAdditions>
	-->

  <!-- optional files or directories to copy or delete from Intermediate/Android/APK before ndk-build -->
  <prebuildCopies>
    <if condition="GCMEnabled">
      <true>
        <copyDir src="$S(PluginDir)/../Java" dst="$S(BuildDir)/src/com/epicgames/unreal" />
      </true>
    </if>
  </prebuildCopies>

  <!-- optional files or directories to copy to Intermediate/Android/APK
	<resourceCopies>
	</resourceCopies>
	-->

  <!-- optional additions to the GameActivity imports in GameActivity.java
	<gameActivityImportAdditions>
	</gameActivityImportAdditions>
	-->

  <!-- optional additions to the GameActivity class in GameActivity.java -->
  <gameActivityClassAdditions>
    <if condition="GCMEnabled">
      <true>
        <insertValue value="	public final String GCMSenderId = &quot;$S(GCMClientSenderID)&quot;;" />
        <insertNewline />
        <insert>
          // Methods that notify the native code about Google Cloud Messaging interactions
          public native void nativeGCMRegisteredForRemoteNotifications(String token);
          public native void nativeGCMFailedToRegisterForRemoteNotifications(String errorMessage);
          public native void nativeGCMReceivedRemoteNotification(String message);

          public void AndroidThunkJava_RegisterForRemoteNotifications()
          {
          // start IntentService to register this application with Google Cloud Messaging
          Intent intent = new Intent(activityContext.getApplicationContext(), RemoteNotificationsRegistrationIntentService.class);
          startService(intent);
          }
          public void AndroidThunkJava_UnregisterForRemoteNotifications()
          {
          }
        </insert>
      </true>
      <false>
        <insert>
          public void AndroidThunkJava_RegisterForRemoteNotifications()
          {
          }
          public void AndroidThunkJava_UnregisterForRemoteNotifications()
          {
          }
        </insert>
      </false>
    </if>
  </gameActivityClassAdditions>

	<gameActivityAllowedRemoteNotificationsAdditions>
		<if condition="GCMEnabled">
			<true>
				<insert>		bPluginEnabled = true;</insert>
				<insertNewline/>
			</true>
		</if>
	</gameActivityAllowedRemoteNotificationsAdditions>

  <!-- optional additions to GameActivity onCreate metadata reading in GameActivity.java
	<gameActivityReadMetadataAdditions>
	</gameActivityReadMetadataAdditions>
	-->

  <!-- optional additions to GameActivity onCreate in GameActivity.java
	<gameActivityOnCreateAdditions>
	</gameActivityOnCreateAdditions>
	-->

  <!-- optional additions to GameActivity onDestroy in GameActivity.java
	<gameActivityOnDestroyAdditions>
	</gameActivityOnDestroyAdditions>
	-->

  <!-- optional additions to GameActivity onStart in GameActivity.java
	<gameActivityOnStartAdditions>
	</gameActivityOnStartAdditions>
  -->

  <!-- optional additions to GameActivity onStop in GameActivity.java
	<gameActivityOnStopAdditions>
	</gameActivityOnStopAdditions>
	-->

  <!-- optional additions to GameActivity onPause in GameActivity.java
	<gameActivityOnPauseAdditions>
	</gameActivityOnPauseAdditions>
	-->

  <!-- optional additions to GameActivity onResume in GameActivity.java
	<gameActivityOnResumeAdditions>
	</gameActivityOnResumeAdditions>
	-->

  <!-- optional additions to GameActivity onActivityResult in GameActivity.java
	<gameActivityOnActivityResultAdditions>
	</gameActivityOnActivityResultAdditions>
	-->

  <!-- optional libraries to load in GameActivity.java before libUnreal.so
	<soLoadLibrary>
	</soLoadLibrary>
	-->
</root>
