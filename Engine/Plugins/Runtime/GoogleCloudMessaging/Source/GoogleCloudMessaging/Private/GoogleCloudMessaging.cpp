// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleCloudMessaging.h"

void IGoogleCloudMessagingModuleInterface::StartupModule()
{
	// This code will execute after your module is loaded into memory (but after global variables are initialized, of course.)
}


void IGoogleCloudMessagingModuleInterface::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#if WITH_EDITOR

DEFINE_LOG_CATEGORY( LogGoogleCloudMessaging );

class FEditorGoogleCloudMessaging : public IGoogleCloudMessagingModuleInterface
{
};

IMPLEMENT_MODULE( FEditorGoogleCloudMessaging, GoogleCloudMessaging )

#endif

