// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleCloudMessaging.h"

#if PLATFORM_ANDROID

#include "Async/TaskGraphInterfaces.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ConfigCacheIni.h"

#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"
#include "Android/AndroidEventManager.h"

DEFINE_LOG_CATEGORY( LogGoogleCloudMessaging );

class FAndroidGoogleCloudMessaging : public IGoogleCloudMessagingModuleInterface
{
};

IMPLEMENT_MODULE(FAndroidGoogleCloudMessaging, GoogleCloudMessaging)

static TArray<uint8> GCMTokenBytes;
static FString GCMErrorMessage = TEXT("");

// registered for remote notifications
JNI_METHOD void Java_com_epicgames_unreal_GameActivity_nativeGCMRegisteredForRemoteNotifications( JNIEnv* jenv, jobject thiz, jstring jGCMToken )
{
	auto GCMTokenLength = jenv->GetStringUTFLength( jGCMToken );
	const char* GCMTokenChars = jenv->GetStringUTFChars( jGCMToken, 0 );

	TArray<uint8> TokenBytes;
	TokenBytes.AddUninitialized( GCMTokenLength );
	FMemory::Memcpy( TokenBytes.GetData(), GCMTokenChars, GCMTokenLength * sizeof( uint8 ) );

	FString GCMToken;
	GCMToken = FString( UTF8_TO_TCHAR( GCMTokenChars ) );

	jenv->ReleaseStringUTFChars( jGCMToken, GCMTokenChars );

	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady( FSimpleDelegateGraphTask::FDelegate::CreateLambda( [=]() {
		UE_LOGF(LogGoogleCloudMessaging, Display, "GCM Registration Token: %ls", *GCMToken);

		GCMTokenBytes = TokenBytes;
		FCoreDelegates::ApplicationRegisteredForRemoteNotificationsDelegate.Broadcast( GCMTokenBytes );
	}), TStatId(), nullptr, ENamedThreads::GameThread );
}

// failed to register for remote notifications
JNI_METHOD void Java_com_epicgames_unreal_GameActivity_nativeGCMFailedToRegisterForRemoteNotifications( JNIEnv* jenv, jobject thiz, jstring jErrorMessage )
{
	FString ErrorMessage;
	const char* GCMErrorMessageChars = jenv->GetStringUTFChars( jErrorMessage, 0 );
	GCMErrorMessage = FString( UTF8_TO_TCHAR( GCMErrorMessageChars ) );
	jenv->ReleaseStringUTFChars( jErrorMessage, GCMErrorMessageChars );

	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady( FSimpleDelegateGraphTask::FDelegate::CreateLambda( [=]() {
		UE_LOGF(LogGoogleCloudMessaging, Display, "GCM Registration Error: %ls", *ErrorMessage);

		GCMErrorMessage = ErrorMessage;
		FCoreDelegates::ApplicationFailedToRegisterForRemoteNotificationsDelegate.Broadcast( GCMErrorMessage );
	}), TStatId(), nullptr, ENamedThreads::GameThread );
}

// received message
JNI_METHOD void Java_com_epicgames_unreal_GameActivity_nativeGCMReceivedRemoteNotification( JNIEnv* jenv, jobject thiz, jstring jMessage )
{
	auto Message = FJavaHelper::FStringFromParam(jenv, jMessage);

	int AppState = 3; // EApplicationState::Active;
	if (FAppEventManager::GetInstance()->IsGamePaused())
	{
		if (FAppEventManager::GetInstance()->IsGameInFocus())
		{
			AppState = 1; // EApplicationState::Inactive;
		}
		else
		{
			AppState = 2; // EApplicationState::Background;
		}
	}

	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady( FSimpleDelegateGraphTask::FDelegate::CreateLambda( [=]() {
		UE_LOGF(LogGoogleCloudMessaging, Display, "GCM AppState = %d, Message : %ls", AppState, *Message);

		FCoreDelegates::ApplicationReceivedRemoteNotificationDelegate.Broadcast( Message, AppState );
	}), TStatId(), nullptr, ENamedThreads::GameThread );
}

#endif
