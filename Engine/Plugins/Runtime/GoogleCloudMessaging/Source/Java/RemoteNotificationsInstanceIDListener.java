// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal;

import android.content.Intent;
import com.google.android.gms.iid.InstanceID;
import com.google.android.gms.iid.InstanceIDListenerService;

public class RemoteNotificationsInstanceIDListener extends InstanceIDListenerService
{
	@Override
	public void onTokenRefresh()
	{
		Intent intent = new Intent( GameActivity.Get().getApplicationContext(), RemoteNotificationsRegistrationIntentService.class );
		startService( intent );
	}
}
