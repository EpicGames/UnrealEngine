// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal;

import android.os.Bundle;
import com.google.android.gms.gcm.GcmListenerService;

import android.app.Notification;
import android.app.NotificationManager;
import android.app.PendingIntent;
import android.content.BroadcastReceiver;
import android.content.Context;
import android.content.Intent;
import androidx.core.app.NotificationCompat;

public class RemoteNotificationsListener extends GcmListenerService
{
	@Override
	public void onMessageReceived( String from, Bundle data ) 
	{
		String message = data.getString( "message" );

		/*
		Intent notificationIntent = new Intent(this, GameActivity.class);
		notificationIntent.setFlags(Intent.FLAG_ACTIVITY_CLEAR_TOP | Intent.FLAG_ACTIVITY_SINGLE_TOP);

		int notificationIconID = context.getResources().getIdentifier("ic_notification", "drawable", context.getPackageName());
		if (notificationIconID == 0)
		{
			notificationIconID = context.getResources().getIdentifier("icon", "drawable", context.getPackageName());
		}
		PendingIntent pendingNotificationIntent = PendingIntent.getActivity(this, notificationIconID, notificationIntent, PendingIntent.FLAG_IMMUTABLE);

		NotificationCompat.Builder builder = new NotificationCompat.Builder(this)
			.setSmallIcon(notificationIconID)
			.setContentIntent(pendingNotificationIntent)
			.setWhen(System.currentTimeMillis())
			.setContentTitle("GCM Message")
			.setStyle(new NotificationCompat.BigTextStyle().bigText(message))
			.setContentText(message);
		if (android.os.Build.VERSION.SDK_INT >= 21)
		{
			builder.setColor(0xff0e1e43);
		}
		Notification notification = builder.build();
		notification.flags |= Notification.FLAG_AUTO_CANCEL;
		notification.defaults |= Notification.DEFAULT_SOUND | Notification.DEFAULT_VIBRATE;

		NotificationManager notificationManager = (NotificationManager)this.getSystemService(Context.NOTIFICATION_SERVICE);
		notificationManager.notify((int)System.currentTimeMillis(), notification);		
		*/
		
		GameActivity.Get().nativeGCMReceivedRemoteNotification( message );
	}
}
