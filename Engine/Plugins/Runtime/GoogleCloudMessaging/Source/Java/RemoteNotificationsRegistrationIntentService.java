// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal;

import java.io.IOException;
import android.app.IntentService;
import android.content.Intent;
import com.google.android.gms.iid.InstanceID;
import com.google.android.gms.gcm.GoogleCloudMessaging;
import com.google.android.gms.gcm.GcmPubSub;

public class RemoteNotificationsRegistrationIntentService extends IntentService
{
  public RemoteNotificationsRegistrationIntentService()
  {
    super( "com.epicgames.unreal.RemoteNotificationsRegistrationIntentService" );
  }

  public RemoteNotificationsRegistrationIntentService( String name )
  {
    super( name );
  }
  
  @Override
  protected void onHandleIntent( Intent intent )
  {
    try
    {
      InstanceID instanceID = InstanceID.getInstance( GameActivity.Get().getApplicationContext() );
      String Token = instanceID.getToken( GameActivity.Get().GCMSenderId, GoogleCloudMessaging.INSTANCE_ID_SCOPE, null );

      // subscribe to default topic: /topics/global
      subscribeTopics( Token );

      GameActivity.Get().nativeGCMRegisteredForRemoteNotifications( Token );
    }
    catch( Exception e )
    {
      GameActivity.Get().nativeGCMFailedToRegisterForRemoteNotifications( "Failed to complete token refresh" );
    }
  }

  private void subscribeTopics( String token ) throws IOException {
    GcmPubSub pubSub = GcmPubSub.getInstance( GameActivity.Get().getApplicationContext() );
    pubSub.subscribe( token, "/topics/global", null );
  }
}
